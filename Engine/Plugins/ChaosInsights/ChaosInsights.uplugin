{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Chaos Insights",
	"Description": "Plugin to gather insights into Chaos",
	"Category": "Insights",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
	{
		"Name": "ChaosInsightsAnalysis",
		"Type": "EditorAndProgram",
		"LoadingPhase": "Default",
		"ProgramAllowList": [
		"UnrealInsights"
			]
	},
	{
		"Name": "ChaosInsightsUI",
		"Type": "EditorAndProgram",
		"LoadingPhase": "Default",
		"ProgramAllowList": [
		"UnrealInsights"
			]
	}
	]
}