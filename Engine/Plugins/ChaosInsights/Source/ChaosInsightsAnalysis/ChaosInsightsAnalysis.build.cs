// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosInsightsAnalysis : ModuleRules
	{
		public ChaosInsightsAnalysis(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(new string[] { "Core" });
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"CoreUObject",
					"TraceLog",
					"TraceAnalysis",
					"TraceServices",
				});
		}
	}
}

