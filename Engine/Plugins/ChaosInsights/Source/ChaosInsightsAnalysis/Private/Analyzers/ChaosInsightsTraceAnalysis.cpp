// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosInsightsTraceAnalysis.h"

#include "Common/ProviderLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Model/LockRegions.h"

namespace ChaosInsightsAnalysis
{

	FLockRegionsTraceAnalyzer::FLockRegionsTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FLockRegionProvider& InRegionProvider)
		: Session(InSession)
		, RegionProvider(InRegionProvider)
	{
	}

	void FLockRegionsTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
	{
		FInterfaceBuilder& Builder = Context.InterfaceBuilder;

		Builder.RouteEvent(static_cast<uint16>(ELockRegionEventId::LockBegin), "Chaos", "LockAcquireBegin");
		Builder.RouteEvent(static_cast<uint16>(ELockRegionEventId::LockAcquire), "Chaos", "LockAcquired");
		Builder.RouteEvent(static_cast<uint16>(ELockRegionEventId::LockEnd), "Chaos", "LockAcquireEnd");
	}

	void FLockRegionsTraceAnalyzer::OnAnalysisEnd()
	{
		TraceServices::FProviderEditScopeLock RegionProviderScopedLock(static_cast<TraceServices::IEditableProvider&>(RegionProvider));
		RegionProvider.OnAnalysisSessionEnded();
	}

	bool FLockRegionsTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
	{
		const FEventData& EventData = Context.EventData;
		const ELockRegionEventId Id = static_cast<ELockRegionEventId>(RouteId);

		switch(Id)
		{
		case ELockRegionEventId::LockBegin:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			bool bWriteLock = EventData.GetValue<bool>("bIsWrite");
			
			TraceServices::FProviderEditScopeLock ProviderLock(RegionProvider);
			RegionProvider.AppendRegionBegin(Context.EventTime.AsSeconds(Cycle), Context.ThreadInfo.GetId(), bWriteLock);

			break;
		}
		case ELockRegionEventId::LockAcquire:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			
			TraceServices::FProviderEditScopeLock ProviderLock(RegionProvider);
			RegionProvider.AppendRegionAcquired(Context.EventTime.AsSeconds(Cycle), Context.ThreadInfo.GetId());

			break;
		}
		case ELockRegionEventId::LockEnd:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			
			TraceServices::FProviderEditScopeLock ProviderLock(RegionProvider);
			RegionProvider.AppendRegionEnd(Context.EventTime.AsSeconds(Cycle), Context.ThreadInfo.GetId());

			break;
		}
		default:
		{
			// Unrequested event ID
			check(false);
		}
		}

		return true;
	}

}
