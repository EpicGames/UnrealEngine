// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"
#include "Templates/SharedPointer.h"

namespace TraceServices
{
	class IAnalysisSession;
}

namespace ChaosInsightsAnalysis
{
	class FLockRegionProvider;

	class FLockRegionsTraceAnalyzer : public UE::Trace::IAnalyzer
	{
	public:
		FLockRegionsTraceAnalyzer(TraceServices::IAnalysisSession& Session, FLockRegionProvider& RegionProvider);

		virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
		virtual void OnAnalysisEnd() override;
		virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

	private:
		enum class ELockRegionEventId : uint16
		{
			LockBegin,
			LockAcquire,
			LockEnd
		};

		TraceServices::IAnalysisSession& Session;
		FLockRegionProvider& RegionProvider;
	};


}
