// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosInsightsAnalysisModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Model/LockRegions.h"
#include "Analyzers/ChaosInsightsTraceAnalysis.h"

namespace ChaosInsightsAnalysis
{
	void FChaosInsightsAnalysisModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
	{
		TSharedPtr<FLockRegionProvider> LockRegionsProvider = MakeShared<FLockRegionProvider>(InSession);
		InSession.AddProvider(GetLockRegionProviderName(), LockRegionsProvider);

		InSession.AddAnalyzer(new FLockRegionsTraceAnalyzer(InSession, *LockRegionsProvider));
	}

	void FChaosInsightsAnalysisModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
	{
		OutModuleInfo.Name = TEXT("ChaosInsightsAnalysis");
		OutModuleInfo.DisplayName = TEXT("Chaos Insights");
	}

	void FChaosInsightsAnalysisModule::StartupModule()
	{
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, this);
	}

	void FChaosInsightsAnalysisModule::ShutdownModule()
	{
		IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, this);
	}

}

IMPLEMENT_MODULE(ChaosInsightsAnalysis::FChaosInsightsAnalysisModule, ChaosInsightsAnalysis);
