// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "TraceServices/ModuleService.h"

class FTabManager;

namespace ChaosInsightsAnalysis
{
	class FChaosInsightsAnalysisModule : public IModuleInterface, TraceServices::IModule
	{
	public:
		// IModuleInterface interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	public:
		// TraceServices::IModule interface
		virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
		virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	};

}
