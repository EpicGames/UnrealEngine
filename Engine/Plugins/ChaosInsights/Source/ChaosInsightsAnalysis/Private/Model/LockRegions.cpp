// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosInsightsAnalysis/Model/LockRegions.h"
#include "Model/LockRegions.h"

#include "Algo/ForEach.h"
#include "Internationalization/Internationalization.h"
#include "TraceServices/Model/Threads.h"

#define LOCTEXT_NAMESPACE "LockRegionProvider"

DEFINE_LOG_CATEGORY(LogChaosInsights)

namespace ChaosInsightsAnalysis
{

	thread_local TraceServices::FProviderLock::FThreadLocalState GLockRegionsProviderLockState;

	FLockRegionProvider::FLockRegionProvider(TraceServices::IAnalysisSession& InSession)
		: Session(InSession)
	{
	}

	FLockRegionProvider::~FLockRegionProvider()
	{

	}

	void FLockRegionProvider::BeginRead() const
	{
		Lock.BeginRead(GLockRegionsProviderLockState);
	}

	void FLockRegionProvider::EndRead() const
	{
		Lock.EndRead(GLockRegionsProviderLockState);
	}

	void FLockRegionProvider::ReadAccessCheck() const
	{
		Lock.ReadAccessCheck(GLockRegionsProviderLockState);
	}

	uint64 FLockRegionProvider::GetRegionCount() const
	{
		ReadAccessCheck();

		uint64 RegionCount = 0;
		for(const FLockRegionLane& Lane : Lanes)
		{
			RegionCount += Lane.Num();
		}
		return RegionCount;
	}

	int32 FLockRegionProvider::GetLaneCount() const
	{
		ReadAccessCheck(); return Lanes.Num();
	}

	const FLockRegionLane* FLockRegionProvider::GetLane(int32 index) const
	{
		ReadAccessCheck();

		if(index < Lanes.Num())
		{
			return &(Lanes[index]);
		}
		return nullptr;
	}

	void FLockRegionProvider::AppendRegionBegin(double Time, uint64 ThreadId, bool bWrite)
	{
		EditAccessCheck();
		
		FLockRegion** FoundRegion = OpenRegionsByThread.Find(ThreadId);

		int32 InitialLockDepth = 0;

		if(FoundRegion)
		{
			(*FoundRegion)->LockCount++;
			(*FoundRegion)->LockDepth++;
		}
		else
		{
			int32 Depth = CalculateRegionDepth(Time);

			while(Depth >= Lanes.Num())
			{
				Lanes.Emplace(Session.GetLinearAllocator());
			}

			TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
			const TraceServices::IThreadProvider& ThreadProvider = TraceServices::ReadThreadProvider(Session);

			Lanes[Depth].Regions.EmplaceBack(FLockRegion
				{
					.BeginTime = Time,
					.Text = Session.StoreString(ThreadProvider.GetThreadName(ThreadId)),
					.Thread = ThreadId,
					.Depth = Depth,
					.LockCount = 1,
					.LockDepth = 1,
					.bIsWrite = bWrite
				});

			OpenRegionsByThread.Add(ThreadId, &Lanes[Depth].Regions.Last());
		}

		UpdateSession(Time);
	}

	void FLockRegionProvider::AppendRegionAcquired(double Time, uint64 ThreadId)
	{
		EditAccessCheck();

		FLockRegion* Region = OpenRegionsByThread.FindChecked(ThreadId);

		if(Time < Region->AcquireTime)
		{
			Region->AcquireTime = Time;
		}

		UpdateSession(Time);
	}

	void FLockRegionProvider::AppendRegionEnd(double Time, uint64 ThreadId)
	{
		EditAccessCheck();

		TMap<uint64_t, FLockRegion*>::TKeyIterator RegionIt = OpenRegionsByThread.CreateKeyIterator(ThreadId);
		FLockRegion& Region = *RegionIt->Value;

		if(--Region.LockDepth == 0)
		{
			RegionIt->Value->EndTime = Time;

			RegionIt.RemoveCurrent();
		}

		UpdateSession(Time);
	}

	void FLockRegionProvider::OnAnalysisSessionEnded()
	{
		EditAccessCheck();

		if(OpenRegionsByThread.Num() > 0)
		{
			UE_LOGF(LogChaosInsights, Warning, "A physics lock event was never closed.");
		}
	}

	int32 FLockRegionProvider::CalculateRegionDepth(double NewBeginTime) const
	{
		constexpr int32 DepthLimit = 100;

		int32 NewDepth = 0;

		// Find first free lane/depth
		while(NewDepth < DepthLimit)
		{
			if(!Lanes.IsValidIndex(NewDepth))
			{
				break;
			}

			const FLockRegion& LastRegion = Lanes[NewDepth].Regions.Last();
			if(LastRegion.EndTime <= NewBeginTime)
			{
				break;
			}
			NewDepth++;
		}

		ensureMsgf(NewDepth < DepthLimit, TEXT("Regions are nested too deep."));

		return NewDepth;
	}

	void FLockRegionProvider::UpdateSession(double InTime)
	{
		TraceServices::FAnalysisSessionEditScope EditScope(Session);
		Session.UpdateDurationSeconds(InTime);
	}

	void FLockRegionProvider::ForEachLane(TFunctionRef<void(const FLockRegionLane&, int32)> Callback) const
	{
		ReadAccessCheck();

		for(int32 LaneIndex = 0; LaneIndex < Lanes.Num(); ++LaneIndex)
		{
			Callback(Lanes[LaneIndex], LaneIndex);
		}
	}

	void FLockRegionProvider::BeginEdit() const
	{
		Lock.BeginWrite(GLockRegionsProviderLockState);
	}

	void FLockRegionProvider::EndEdit() const
	{
		Lock.EndWrite(GLockRegionsProviderLockState);
	}

	void FLockRegionProvider::EditAccessCheck() const
	{
		Lock.WriteAccessCheck(GLockRegionsProviderLockState);
	}

	bool FLockRegionProvider::ForEachRegionInRange(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FLockRegion&)> Callback) const
	{
		ReadAccessCheck();

		if(IntervalStart > IntervalEnd)
		{
			return false;
		}

		for(const FLockRegionLane& Lane : Lanes)
		{
			if(!Lane.ForEachRegionInRange(IntervalStart, IntervalEnd, Callback))
			{
				return false;
			}
		}

		return true;
	}

	FLockRegionLane::FLockRegionLane(TraceServices::ILinearAllocator& InAllocator) 
		: Regions(InAllocator, 512)
	{

	}

	int32 FLockRegionLane::Num() const
	{
		return static_cast<int32>(Regions.Num());
	}

	bool FLockRegionLane::ForEachRegionInRange(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FLockRegion&)> Callback) const
	{
		struct FTimeProjections
		{
			static double Begin(const FLockRegion& InRegion)
			{
				return InRegion.BeginTime;
			}

			static double End(const FLockRegion& InRegion)
			{
				return InRegion.EndTime;
			}
		};

		const FInt32Interval OverlapRange = GetElementRangeOverlappingGivenRange<FLockRegion>(Regions, IntervalStart, IntervalEnd, &FTimeProjections::Begin, &FTimeProjections::End);

		if(OverlapRange.Min == -1)
		{
			return true;
		}

		for(int32 Index = OverlapRange.Min; Index <= OverlapRange.Max; ++Index)
		{
			if(!Callback(Regions[Index]))
			{
				return false;
			}
		}

		return true;
	}

	FName GetLockRegionProviderName()
	{
		static const FName Name("LockRegionProvider");
		return Name;
	}

	ILockRegionProvider::~ILockRegionProvider() = default;

	const ILockRegionProvider& ReadRegionProvider(const TraceServices::IAnalysisSession& Session)
	{
		return *Session.ReadProvider<ILockRegionProvider>(GetLockRegionProviderName());
	}

	IEditableLockRegionProvider::~IEditableLockRegionProvider() = default;

	IEditableLockRegionProvider& EditRegionProvider(TraceServices::IAnalysisSession& Session)
	{
		return *Session.EditProvider<IEditableLockRegionProvider>(GetLockRegionProviderName());
	}

}

#undef LOCTEXT_NAMESPACE
