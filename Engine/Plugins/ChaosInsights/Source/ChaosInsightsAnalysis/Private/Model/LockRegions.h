// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/ProviderLock.h"
#include "Templates/SharedPointer.h"
#include "ChaosInsightsAnalysis/Model/LockRegions.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChaosInsights, Display, All)

namespace TraceServices
{
	class FAnalysisSessionLock;
	class FStringStore;
}

namespace ChaosInsightsAnalysis
{
	extern thread_local TraceServices::FProviderLock::FThreadLocalState GLockRegionsProviderLockState;

	class FLockRegionProvider : public ILockRegionProvider, public IEditableLockRegionProvider
	{
	public:
		explicit FLockRegionProvider(TraceServices::IAnalysisSession& Session);
		virtual ~FLockRegionProvider() override;

		//////////////////////////////////////////////////
		// Read operations

		virtual void BeginRead() const override;
		virtual void EndRead() const override;
		virtual void ReadAccessCheck() const override;

		virtual uint64 GetRegionCount() const override;
		virtual int32 GetLaneCount()  const override;

		virtual const FLockRegionLane* GetLane(int32 Index) const override;

		virtual bool ForEachRegionInRange(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FLockRegion&)> Callback) const override;
		virtual void ForEachLane(TFunctionRef<void(const FLockRegionLane&, const int32)> Callback) const override;

		//////////////////////////////////////////////////
		// Edit operations

		virtual void BeginEdit() const override;
		virtual void EndEdit() const override;
		virtual void EditAccessCheck() const override;

		virtual void AppendRegionBegin(double Time, uint64 ThreadId, bool bWrite) override;
		virtual void AppendRegionAcquired(double Time, uint64 ThreadId) override;
		virtual void AppendRegionEnd(double Time, uint64 ThreadId) override;
		
		virtual void OnAnalysisSessionEnded() override;

		//////////////////////////////////////////////////

	private:
		int32 CalculateRegionDepth(double NewBeginTime) const;
		void UpdateSession(double InTime);
		
		TraceServices::IAnalysisSession& Session;
		mutable TraceServices::FProviderLock Lock;

		TMap<uint64_t, FLockRegion*> OpenRegionsByThread;
		TArray<FLockRegionLane> Lanes;
	};

}
