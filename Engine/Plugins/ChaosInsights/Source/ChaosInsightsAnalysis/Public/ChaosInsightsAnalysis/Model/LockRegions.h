// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/PagedArray.h" // TraceServices
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "UObject/NameTypes.h"

#define UE_API CHAOSINSIGHTSANALYSIS_API

namespace ChaosInsightsAnalysis
{

	struct FLockRegion
	{
		// Time that the caller attempted to take the lock
		double BeginTime = std::numeric_limits<double>::infinity();

		// Time that the lock was actually acquired (including waiting on the lock if it was already taken)
		double AcquireTime = std::numeric_limits<double>::infinity();

		// Time that the caller relinquished the lock
		double EndTime = std::numeric_limits<double>::infinity();

		// Name of the thread that took the lock
		const TCHAR* Text = nullptr;

		// The local thread Id for the thread that took the lock
		uint64 Thread;

		// UI depth
		int32 Depth = -1;

		// Number of times a lock was taken during the region
		int32 LockCount = 0;

		// Tracking for the lock depth to combine all recursive locks into one region
		int32 LockDepth = 0;

		// Whether this is a write-lock (as opposed to a read-lock)
		bool bIsWrite = false;
	};

	class FLockRegionLane
	{
		friend class FLockRegionProvider;

	public:
		UE_API FLockRegionLane(TraceServices::ILinearAllocator& InAllocator);

		UE_API int32 Num() const;

		UE_API bool ForEachRegionInRange(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FLockRegion&)> Callback) const;

	private:
		TraceServices::TPagedArray<FLockRegion> Regions;
	};

	class ILockRegionProvider : public TraceServices::IProvider
	{
	public:
		UE_API virtual ~ILockRegionProvider() override;

		virtual uint64 GetRegionCount() const = 0;
		virtual int32 GetLaneCount() const = 0;
		virtual const FLockRegionLane* GetLane(int32 Index) const = 0;

		virtual bool ForEachRegionInRange(double IntervalStart, double IntervalEnd, TFunctionRef<bool(const FLockRegion&)> Callback) const = 0;
		virtual void ForEachLane(TFunctionRef<void(const FLockRegionLane&, const int32)> Callback) const = 0;
	};

	class IEditableLockRegionProvider : public TraceServices::IEditableProvider
	{
	public:
		virtual ~IEditableLockRegionProvider() override;

		virtual void AppendRegionBegin(double Time, uint64 ThreadId, bool bIsWrite) = 0;
		virtual void AppendRegionAcquired(double Time, uint64 ThreadId) = 0;
		virtual void AppendRegionEnd(double Time, uint64 ThreadId) = 0;

		virtual void OnAnalysisSessionEnded() = 0;
	};

	////////////////////////////////////////////////////////////////////////////////////////////////////

	CHAOSINSIGHTSANALYSIS_API FName GetLockRegionProviderName();
	CHAOSINSIGHTSANALYSIS_API const ILockRegionProvider& ReadRegionProvider(const TraceServices::IAnalysisSession& Session);
	CHAOSINSIGHTSANALYSIS_API IEditableLockRegionProvider& EditRegionProvider(TraceServices::IAnalysisSession& Session);

} // namespace TimingRegionsAnalysis

#undef UE_API
