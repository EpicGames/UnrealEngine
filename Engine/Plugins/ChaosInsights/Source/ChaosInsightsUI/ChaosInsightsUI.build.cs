// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosInsightsUI : ModuleRules
	{
		public ChaosInsightsUI(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"SlateCore",
				"Slate",
				"InputCore",
				"TraceInsights",
				"TraceInsightsCore",
				"TraceServices",
				"ChaosInsightsAnalysis",
			});
		}
	}
}

