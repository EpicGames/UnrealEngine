// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosInsightsUIModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Insights/ITimingViewExtender.h"

namespace ChaosInsights 
{

	void FChaosInsightsUIModule::StartupModule()
	{
		IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}

	void FChaosInsightsUIModule::ShutdownModule()
	{
		IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
	}

}

IMPLEMENT_MODULE(ChaosInsights::FChaosInsightsUIModule, ChaosInsightsUI);
