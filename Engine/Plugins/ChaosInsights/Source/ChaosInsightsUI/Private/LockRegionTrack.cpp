// Copyright Epic Games, Inc. All Rights Reserved.

#include "LockRegionTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Common/ProviderLock.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ViewModels/TimingEventSearch.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "InsightsCore/Common/TimeUtils.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "LockRegionsTimingTrack"

class IUnrealInsightsModule;
class FTimingEvent;

namespace ChaosInsights
{
	namespace Colors
	{
		static constexpr FColor WaitingColor{255, 15, 15};
		static constexpr FColor AcquiredColorRead{240, 240, 100};
		static constexpr FColor AcquiredColorWrite{15, 255, 15};
	}

	FLockRegionsSharedState::~FLockRegionsSharedState() = default;

	void FLockRegionsSharedState::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{
		if (TimingView == nullptr)
		{
			TimingView = &InSession;
		}

		if(!IsCurrentSession(InSession))
		{
			return;
		}

		LockRegionsTrack.Reset();
	}

	void FLockRegionsSharedState::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
	{
		if(!IsCurrentSession(InSession))
		{
			return;
		}

		TimingView = nullptr;

		LockRegionsTrack.Reset();
	}

	void FLockRegionsSharedState::Tick(UE::Insights::Timing::ITimingViewSession& InSession,
		const TraceServices::IAnalysisSession& InAnalysisSession)
	{
		if(!IsCurrentSession(InSession))
		{
			return;
		}

		if(!LockRegionsTrack.IsValid())
		{
			LockRegionsTrack = MakeShared<FLockRegionsTrack>(*this);
			LockRegionsTrack->SetOrder(FTimingTrackOrder::First);
			LockRegionsTrack->SetVisibilityFlag(true);
			InSession.AddScrollableTrack(LockRegionsTrack);
		}
	}

	void FLockRegionsSharedState::ShowHideRegionsTrack()
	{
		bShowHideRegionsTrack = !bShowHideRegionsTrack;

		if(LockRegionsTrack.IsValid())
		{
			LockRegionsTrack->SetVisibilityFlag(bShowHideRegionsTrack);
		}

		if(bShowHideRegionsTrack)
		{
			LockRegionsTrack->SetDirtyFlag();
		}
	}

	bool FLockRegionsSharedState::IsRegionsTrackVisible() const
	{
		return bShowHideRegionsTrack;
	}

	bool FLockRegionsSharedState::IsCurrentSession(UE::Insights::Timing::ITimingViewSession& Session)
	{
		return &Session == TimingView;
	}

	INSIGHTS_IMPLEMENT_RTTI(FLockRegionsTrack);

	FLockRegionsTrack::FLockRegionsTrack(FLockRegionsSharedState& InSharedState)
		: FTimingEventsTrack(TEXT("Physics Scene Locks")), SharedState(InSharedState)
	{
		IUnrealInsightsModule& UnrealInsightsModule = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
		AnalysisSession = UnrealInsightsModule.GetAnalysisSession();
	}

	FLockRegionsTrack::~FLockRegionsTrack() = default;

	void FLockRegionsTrack::InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const
	{
		if(InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FTimingEvent>())
		{
			const FTimingEvent& TooltipEvent = InTooltipEvent.As<FTimingEvent>();

			auto MatchEvent = [this, &TooltipEvent](double InStartTime, double InEndTime, uint32 InDepth)
				{
					return InDepth == TooltipEvent.GetDepth()
						&& InStartTime == TooltipEvent.GetStartTime()
						&& InEndTime == TooltipEvent.GetEndTime();
				};

			FTimingEventSearchParameters SearchParameters(TooltipEvent.GetStartTime(), TooltipEvent.GetEndTime(), ETimingEventSearchFlags::StopAtFirstMatch, MatchEvent);

			auto SearchLambda = [this, &InOutTooltip, &TooltipEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const ChaosInsightsAnalysis::FLockRegion& InRegion)
				{
					InOutTooltip.Reset();
					InOutTooltip.AddTitle(InRegion.Text, FLinearColor::White);
					InOutTooltip.AddNameValueTextLine(TEXT("Type:"), InRegion.bIsWrite ? TEXT("Write") : TEXT("Read"));
					InOutTooltip.AddNameValueTextLine(TEXT("Wait Duration:"), UE::Insights::FormatTimeAuto(InRegion.AcquireTime - InRegion.BeginTime));
					InOutTooltip.AddNameValueTextLine(TEXT("Exec Duration:"), UE::Insights::FormatTimeAuto(InRegion.EndTime - InRegion.AcquireTime));
					InOutTooltip.AddNameValueTextLine(TEXT("Max Lock Depth:"), FString::FromInt(InRegion.LockCount));
					InOutTooltip.UpdateLayout();
				};

			FindRegionEvent(SearchParameters, SearchLambda);
		}
	}

	void FLockRegionsTrack::BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder,
		const ITimingTrackUpdateContext& Context)
	{
		const FTimingTrackViewport& Viewport = Context.GetViewport();

		const ChaosInsightsAnalysis::ILockRegionProvider& RegionProvider = ChaosInsightsAnalysis::ReadRegionProvider(*AnalysisSession);
		TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

		int32 CurDepth = 0;
		RegionProvider.ForEachLane([this, Viewport, &CurDepth, &Builder](const ChaosInsightsAnalysis::FLockRegionLane& Lane, const int32 Depth)
			{
				bool RegionHadEvents = false;
				Lane.ForEachRegionInRange(Viewport.GetStartTime(), Viewport.GetEndTime(), [&Builder, &RegionHadEvents, &CurDepth](const ChaosInsightsAnalysis::FLockRegion& Region) -> bool
					{
						RegionHadEvents = true;

						const uint32 WaitColorBytes = Colors::WaitingColor.DWColor();
						const uint32 AcquiredColorBytes = Region.bIsWrite ? Colors::AcquiredColorWrite.DWColor() : Colors::AcquiredColorRead.DWColor();

						Builder.AddEvent(Region.BeginTime, Region.AcquireTime, CurDepth, Region.Text, 0, WaitColorBytes);
						Builder.AddEvent(Region.AcquireTime, Region.EndTime, CurDepth, Region.Text, 0, AcquiredColorBytes);

						return true;
					});

				if(RegionHadEvents)
				{
					CurDepth++;
				}
			});
	}

	const TSharedPtr<const ITimingEvent> FLockRegionsTrack::SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const
	{
		TSharedPtr<const ITimingEvent> FoundEvent;

		auto SearchLambda = [this, &FoundEvent](double InFoundStartTime, double InFoundEndTime, uint32 InFoundDepth, const ChaosInsightsAnalysis::FLockRegion& InEvent)
			{
				FoundEvent = MakeShared<FTimingEvent>(SharedThis(this), InFoundStartTime, InFoundEndTime, InFoundDepth, reinterpret_cast<uint64>(InEvent.Text));
			};

		FindRegionEvent(InSearchParameters, SearchLambda);

		return FoundEvent;
	}

	bool FLockRegionsTrack::FindRegionEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const ChaosInsightsAnalysis::FLockRegion&)> InFoundPredicate) const
	{
		{
			// If the query start time is larger than the end of the session return false.
			TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession.Get());
			if(AnalysisSession.IsValid() && InParameters.StartTime > AnalysisSession->GetDurationSeconds())
			{
				return false;
			}
		}

		auto SearchLambda = [this](TTimingEventSearch<ChaosInsightsAnalysis::FLockRegion>::FContext& InContext)
			{
				const ChaosInsightsAnalysis::ILockRegionProvider& RegionProvider = ChaosInsightsAnalysis::ReadRegionProvider(*AnalysisSession);
				TraceServices::FProviderReadScopeLock RegionProviderScopedLock(RegionProvider);

				auto RegionSearchLambda = [&InContext](const ChaosInsightsAnalysis::FLockRegion& Region)
					{
						InContext.Check(Region.BeginTime, Region.EndTime, Region.Depth, Region);

						if(!InContext.ShouldContinueSearching())
						{
							return false;
						}

						return true;
					};

				RegionProvider.ForEachRegionInRange(InContext.GetParameters().StartTime, InContext.GetParameters().EndTime, RegionSearchLambda);
			};

		return TTimingEventSearch<ChaosInsightsAnalysis::FLockRegion>::Search(InParameters, SearchLambda, TTimingEventSearch<ChaosInsightsAnalysis::FLockRegion>::NoFilter, InFoundPredicate, TTimingEventSearch<ChaosInsightsAnalysis::FLockRegion>::NoMatch);
	}
}

#undef LOCTEXT_NAMESPACE
