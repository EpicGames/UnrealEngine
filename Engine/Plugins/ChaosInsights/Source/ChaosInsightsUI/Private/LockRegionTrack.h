// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ITimingViewExtender.h"
#include "Insights/ViewModels/TimingEventsTrack.h"
#include "ChaosInsightsAnalysis/Model/LockRegions.h"

namespace ChaosInsights
{

	class FLockRegionsTrack;

	class FLockRegionsSharedState : public UE::Insights::Timing::ITimingViewExtender, public TSharedFromThis<FLockRegionsSharedState>
	{
		friend class FLockRegionsTrack;

	public:
		virtual ~FLockRegionsSharedState() override;

		// ITimingViewExtender interface
		virtual void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
		virtual void OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession) override;
		virtual void Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession) override;

		void ShowHideRegionsTrack();
		bool IsRegionsTrackVisible() const;

	private:
		bool IsCurrentSession(UE::Insights::Timing::ITimingViewSession& Session);

		UE::Insights::Timing::ITimingViewSession* TimingView = nullptr;
		TSharedPtr<FLockRegionsTrack> LockRegionsTrack;
		bool bShowHideRegionsTrack = true;
	};

	class FLockRegionsTrack : public FTimingEventsTrack
	{
		INSIGHTS_DECLARE_RTTI(FLockRegionsTrack, FTimingEventsTrack)

	public:
		FLockRegionsTrack(FLockRegionsSharedState& InSharedState);
		virtual ~FLockRegionsTrack() override;

		virtual void InitTooltip(FTooltipDrawState& InOutTooltip, const ITimingEvent& InTooltipEvent) const override;
		virtual const TSharedPtr<const ITimingEvent> SearchEvent(const FTimingEventSearchParameters& InSearchParameters) const override;
		virtual void BuildDrawState(ITimingEventsTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context) override;

	protected:
		bool FindRegionEvent(const FTimingEventSearchParameters& InParameters, TFunctionRef<void(double, double, uint32, const ChaosInsightsAnalysis::FLockRegion&)> InFoundPredicate) const;

	private:
		FLockRegionsSharedState& SharedState;
		TSharedPtr<const TraceServices::IAnalysisSession> AnalysisSession;
	};

}
